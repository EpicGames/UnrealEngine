// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "AudioLinkSettingsAbstract.generated.h"

class UAudioLinkSettingsAbstract;

/**
  * This interface should be used to provide a non-uclass version of the data described in
  * your implementation of UAudioLinkSettingsAbstract
  */
class IAudioLinkSettingsProxy
{
public:
	virtual ~IAudioLinkSettingsProxy() = default;
protected:
#if WITH_EDITOR
	friend class UAudioLinkSettingsAbstract;
	virtual void RefreshFromSettings(UAudioLinkSettingsAbstract* InSettings, FPropertyChangedEvent& InPropertyChangedEvent) = 0;
#endif //WITH_EDITOR
};

/**
  * This opaque class should be used for specifying settings for how audio should be
  * send to an external audio link.
  */
UCLASS(config = Engine, abstract, editinlinenew, BlueprintType, MinimalAPI)
class UAudioLinkSettingsAbstract : public UObject
{
	GENERATED_BODY()

public:
	using FSharedSettingsProxyPtr = TSharedPtr<IAudioLinkSettingsProxy, ESPMode::ThreadSafe>;

	virtual FName GetFactoryName() const PURE_VIRTUAL(UAudioLinkSettingsAbstract::GetFactoryName, return {};);
	
	virtual const FSharedSettingsProxyPtr& GetProxy() const
	{
		if (!ProxyInstance.IsValid())
		{
			ProxyInstance = MakeProxy();
		}
		return ProxyInstance;
	}

	template<typename T> 
	auto GetCastProxy() const
	{
		return StaticCastSharedPtr<T>(GetProxy());
	}

protected:
#if WITH_EDITOR	
	void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
	{
		GetProxy()->RefreshFromSettings(this, PropertyChangedEvent);
	}
#endif //WITH_EDITOR

	virtual FSharedSettingsProxyPtr MakeProxy() const PURE_VIRTUAL(UAudioLinkSettingsAbstract::MakeProxy, return {};);
	mutable FSharedSettingsProxyPtr ProxyInstance;
};

