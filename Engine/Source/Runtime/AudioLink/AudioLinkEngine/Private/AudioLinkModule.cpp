// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "AudioLinkLog.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogAudioLink);

class FAudioLinkModule : public IModuleInterface
{
public:
	// Begin IModuleInterface
	virtual void StartupModule() override
	{
	}
	virtual void ShutdownModule() override
	{
	}
};

IMPLEMENT_MODULE(FAudioLinkModule, AudioLinkEngine);
