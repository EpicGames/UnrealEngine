// Copyright Epic Games, Inc. All Rights Reserved.

#include "BufferedListenerBase.h"
#include "AudioLinkLog.h"

FBufferedListenerBase::FBufferedListenerBase(int32 InDefaultCircularBufferSize)
	: CircularBuffer(InDefaultCircularBufferSize)
{
	CircularBuffer.SetCapacity(InDefaultCircularBufferSize);
}

/** CONSUMER Thread. This will be called by the consuming data of the buffers. */
bool FBufferedListenerBase::PopBuffer(float* InBuffer, int32 InBufferSizeInSamples, int32& OutSamplesWritten)
{
	FScopeLock Lock(&CircularBuffer.GetCriticialSection());
	OutSamplesWritten = CircularBuffer.Pop(InBuffer, InBufferSizeInSamples);
	int32 NumSamplesRemaining = CircularBuffer.Num();

	// Return true if any data remaining, which signals the consumer to shutdown if false.
	// However if we're still running, we could be starving, so only signal stopped if we've exhausted.
	return (bStopping && NumSamplesRemaining > 0) || bStarted;
}

/** CONSUMER Thread. This will be called by the consuming data of the buffers. */
bool FBufferedListenerBase::GetFormat(IBufferedAudioOutput::FBufferFormat& OutFormat) const
{
	FReadScopeLock ReadLock(FormatKnownRwLock);
	if (KnownFormat)
	{
		OutFormat = *KnownFormat;
		return true; // success.
	}
	return false;
}

void FBufferedListenerBase::SetFormatKnownDelegate(FOnFormatKnown InFormatKnownDelegate)
{
	OnFormatKnown = InFormatKnownDelegate;
}

/** AUDIO MIXER THREAD. */
void FBufferedListenerBase::OnBufferReceived(const FBufferFormat& InFormat, TArrayView<const float> InBuffer)
{
	// Keep track of if we need to fire the delegate, so we can do it outside of a lock.
	bool bFireFormatKnownDelegate = false;

	// Do we know the format yet? (do this under a read-only lock, unless we need to change it).
	{
		// Read lock to check if we know the state.
		FRWScopeLock Lock(FormatKnownRwLock, SLT_ReadOnly);

		// Format known?
		if (!KnownFormat)
		{
			{
				// Upgrade lock to Write lock. (Safe here as we are the only writer, and only readers can race between.).
				Lock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();

				// Write this format as our known format.
				KnownFormat = InFormat;

				// Remember to fire the delegate.
				bFireFormatKnownDelegate = true;
			}
		}
		else
		{
			// Sanity check they haven't changed on the source since it started.
			// Note, in this case it is okay to be off a couple samples if we are doing sample rate conversion	
			ensure(InFormat.NumChannels == KnownFormat->NumChannels &&
				FMath::Abs(InFormat.NumSamplesPerBlock - KnownFormat->NumSamplesPerBlock) <= InFormat.NumChannels &&
				InFormat.NumSamplesPerSec == KnownFormat->NumSamplesPerSec);
		}
	}

	// Fire format known delegate. (important this is done outside of the read/write lock above as it calls GetFormat, which needs a read lock).
	if (bFireFormatKnownDelegate)
	{
		// Broadcast to consumer that we know our format and block rate.
		OnFormatKnown.ExecuteIfBound(InFormat);
	}

	int32 SamplesPushed = 0;
	{
		// Lock outside for the sake of atomic logging
		FScopeLock Lock(&CircularBuffer.GetCriticialSection());

		// Push the data into the circular buffer.
		SamplesPushed = CircularBuffer.Push(InBuffer.GetData(), InBuffer.Num());

		UE_LOGF(LogAudioLink, VeryVerbose,
			"FBufferedListenerBase::OnBufferReceived()(post-push), SamplesPushed=%d, InBuffer.Num()=%d, LocklessCircularBuffer.Num()=%d, LocklessCircularBuffer.Remainder()=%d, LocklessCircularBuffer.Capacity()=%d, This=0x%p",
			SamplesPushed, InBuffer.Num(), CircularBuffer.Num(), CircularBuffer.Remainder(), CircularBuffer.GetCapacity(), this);
	}

	// Warn of not enough space in circular buffer, unless we're overwriting silence.
	if (SamplesPushed < InBuffer.Num())
	{
		// Prevent log spam by limiting to 1:100 logs.
		static const int32 NumLogMessagesToSkip = 100;
		static int32 LogPacifier = 0;
		UE_CLOGF(LogPacifier++ % NumLogMessagesToSkip == 0, LogAudioLink,
			Verbose, "FBufferedListenerBase: Overflow by '%d' Samples in Buffer Listener", InBuffer.Num() - SamplesPushed);
	}
}

void FBufferedListenerBase::ResetFormat()
{
	FWriteScopeLock WriteLock(FormatKnownRwLock);
	KnownFormat.Reset();
}

void FBufferedListenerBase::SetFormat(const FBufferFormat& InFormat)
{
	FWriteScopeLock WriteLock(FormatKnownRwLock);
	KnownFormat = InFormat;
}

bool FBufferedListenerBase::IsStartedNonAtomic() const
{
	return bStarted;
}

bool FBufferedListenerBase::TrySetStartedFlag()
{
	// We expect not to be started.
	bool bExpected = false;
	bool bDesired = true;
	return bStarted.compare_exchange_strong(bExpected, bDesired);
}

bool FBufferedListenerBase::TryUnsetStartedFlag()
{
	// We expect be started.
	bool bExpected = true;
	bool bDesired = false;
	return bStarted.compare_exchange_strong(bExpected, bDesired);
}

bool FBufferedListenerBase::TrySetStoppingFlag()
{
	bool bExpected = false;
	bool bDesired = true;
	return bStopping.compare_exchange_strong(bExpected, bDesired);
}

void FBufferedListenerBase::PushSilence(int32 InNumSamplesOfSilence)
{
	CircularBuffer.PushZeros(InNumSamplesOfSilence);
}

void FBufferedListenerBase::Reserve(int32 InNumSamplesToReserve, int32 InNumSamplesOfSilence)
{
	// This Zeros the buffer also so should only be done at the start.
	CircularBuffer.SetCapacity(InNumSamplesToReserve);

	// Optionally add silence into the buffer, this will cause latency.
	if (InNumSamplesOfSilence > 0)
	{
		PushSilence(InNumSamplesOfSilence);
	}
}

int32 FBufferedListenerBase::GetNumBufferedSamples()
{
	// Lock for safety
	FScopeLock Lock(&CircularBuffer.GetCriticialSection());
	return CircularBuffer.Num();
}

void FBufferedListenerBase::FlushAndResetFormat()
{
	CircularBuffer.Flush();
	ResetFormat();
}

void FBufferedListenerBase::FlushAndEnsureCapacity(int32 InMinCapacity)
{
	CircularBuffer.EnsureCapacity(InMinCapacity);
	ResetFormat();
}
