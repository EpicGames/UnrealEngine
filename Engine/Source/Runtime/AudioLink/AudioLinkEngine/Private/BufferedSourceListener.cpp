// Copyright Epic Games, Inc. All Rights Reserved.

#include "BufferedSourceListener.h"

/** Buffered Source Listener. */
FBufferedSourceListener::FBufferedSourceListener(int32 InDefaultCircularBufferSize)
	: FBufferedListenerBase{ InDefaultCircularBufferSize }
{
}

FBufferedSourceListener::~FBufferedSourceListener()
{
	// No need to stop as its only record keeping and nothing to unregister.
}

bool FBufferedSourceListener::Start(FAudioDevice*)
{
	return TrySetStartedFlag();
}

void FBufferedSourceListener::Stop(FAudioDevice*)
{
	TrySetStoppingFlag();
	TryUnsetStartedFlag();
}

/** AUDIO MIXER THREAD. When a source is finished and returned to the pool, this call will be called. */
void FBufferedSourceListener::OnSourceReleased(const int32 InSourceId)
{	
	if (InSourceId == CurrentSourceId)
	{
		// Fire delegate to any interested parties.
		IBufferedAudioOutput::FBufferStreamEnd Params;
		Params.Id = CurrentSourceId;
		OnBufferStreamEndDelegate.ExecuteIfBound(Params);

		// Reset our source id.
		CurrentSourceId = INDEX_NONE;

		// Mark us stopping.
		Stop(nullptr);

	}
}

void FBufferedSourceListener::SetBufferStreamEndDelegate(FOnBufferStreamEnd InBufferStreamEndDelegate)
{
	OnBufferStreamEndDelegate = InBufferStreamEndDelegate;
}

/** AUDIO MIXER THREAD. New Audio buffers from the active sources enter here. */
void FBufferedSourceListener::OnNewBuffer(const ISourceBufferListener::FOnNewBufferParams& InParams)
{
	if (IsStartedNonAtomic()) 
	{
		// Make sure caller is sane.
		check(InParams.AudioData);
		check(InParams.NumChannels > 0);
		check(InParams.NumSamples > 0);
		check(InParams.SampleRate > 0);
		check(InParams.SourceId >= 0);
		
		// Multiple sources can call this same instance, so make sure we're only listening to one.
		int32 OriginalCurrentSourceId = CurrentSourceId.load(std::memory_order_relaxed);	
		if (OriginalCurrentSourceId == INDEX_NONE)
		{
			// Exchange CurrentSourceId, which is INDEX_NONE with InSourceId  (unless another thread beats us to it). 
			int32 NotSetSourceId = INDEX_NONE;
			if (!CurrentSourceId.compare_exchange_strong(/* expected */ NotSetSourceId, /* desired */ InParams.SourceId, std::memory_order_relaxed))
			{
				return;
			}
		}

		// Call to base class to handle.
		FBufferFormat NewFormat;
		NewFormat.NumChannels = InParams.NumChannels;
		NewFormat.NumSamplesPerBlock = InParams.NumSamples;
		NewFormat.NumSamplesPerSec = InParams.SampleRate;
		OnBufferReceived(NewFormat, MakeArrayView(InParams.AudioData, InParams.NumSamples));
	}
}
