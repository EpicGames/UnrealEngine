// Copyright Epic Games, Inc. All Rights Reserved.

#include "BufferedSubmixListener.h"
#include "AudioMixerDevice.h"

namespace BufferedSubmixListenerPrivate
{
	static Audio::FDeviceId InvalidAudioDeviceId = static_cast<Audio::FDeviceId>(INDEX_NONE);
}

/** Buffered Submix Listener. */
FBufferedSubmixListener::FBufferedSubmixListener(int32 InDefaultCircularBufferSize, bool bInZeroInputBuffer, const FString* InName)
	: FBufferedListenerBase{ InDefaultCircularBufferSize }
	, bZeroInputBuffer{ bInZeroInputBuffer }
	, DeviceId{ BufferedSubmixListenerPrivate::InvalidAudioDeviceId }
{
	if (InName)
	{
		Name = *InName;
	}
}

FBufferedSubmixListener::~FBufferedSubmixListener()
{
	// We should have been unregistered.
	check(DeviceId == BufferedSubmixListenerPrivate::InvalidAudioDeviceId);
	check(!IsStartedNonAtomic());
}

const FString& FBufferedSubmixListener::GetListenerName() const
{
	return Name;
}

bool FBufferedSubmixListener::Start(FAudioDevice* InAudioDevice, USoundSubmix* InSubmix)
{
	if (ensure(InAudioDevice))
	{
		if (TrySetStartedFlag())
		{
			USoundSubmix* QualifiedSubmix = InSubmix ? TObjectPtr<USoundSubmix>(InSubmix) : TObjectPtr<USoundSubmix>(&InAudioDevice->GetMainSubmixObject());
			
			check(IsInAudioThread() || IsInGameThread());
			if (ensure(IsValid(QualifiedSubmix)))
			{
				InAudioDevice->RegisterSubmixBufferListener(AsShared(), *QualifiedSubmix);
				DeviceId = InAudioDevice->DeviceID;
				Submix = QualifiedSubmix;
				return true;
			}

		}
	}
	return false;
}

bool FBufferedSubmixListener::Start(FAudioDevice* InAudioDevice)
{
	if (ensure(InAudioDevice))
	{
		return Start(InAudioDevice, &InAudioDevice->GetMainSubmixObject());
	}
	return false;
}

void FBufferedSubmixListener::Stop(FAudioDevice* InAudioDevice)
{
	if (ensure(InAudioDevice))
	{
		if (ensure(InAudioDevice->DeviceID == DeviceId))
		{
			if (TryUnsetStartedFlag())
			{
				check(IsInAudioThread() || IsInGameThread());

				// In audio thread, GC cannot interrupt this task
				if (USoundSubmix* QualifiedSubmix = Submix.Get())
				{
					InAudioDevice->UnregisterSubmixBufferListener(AsShared(), *QualifiedSubmix);
				}

				DeviceId = BufferedSubmixListenerPrivate::InvalidAudioDeviceId;
				Submix = nullptr;
			}
		}
	}
}

void FBufferedSubmixListener::OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 InNumSamples, int32 InNumChannels, const int32 InSampleRate, double /*AudioClock*/)
{
	if (IsStartedNonAtomic())
	{
		// Call to base class to handle.
		FBufferFormat NewFormat;
		NewFormat.NumChannels = InNumChannels;
		NewFormat.NumSamplesPerBlock = InNumSamples;
		NewFormat.NumSamplesPerSec = InSampleRate;
		OnBufferReceived(NewFormat, MakeArrayView(AudioData, InNumSamples));

		// Optionally, zero the buffer if we're asked to. This in the case where we're running both Unreal+Consumer renderers at once.
		// NOTE: this is dangerous as there's a chance we're not the only listener registered on this Submix. And will cause
		// listeners after us to have a silent buffer. Use with caution. 

		if (bZeroInputBuffer)
		{
			FMemory::Memzero(AudioData, sizeof(float) * InNumSamples);
		}
	}
}
