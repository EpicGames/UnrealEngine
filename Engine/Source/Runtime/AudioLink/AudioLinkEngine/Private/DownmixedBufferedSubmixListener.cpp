// Copyright Epic Games, Inc. All Rights Reserved.

#include "DownmixedBufferedSubmixListener.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/ChannelMap.h"
#include "DSP/Dsp.h"
#include "SampleBuffer.h"
#include "Sound/SoundSubmix.h"

namespace DownmixedBufferedSubmixListenerPrivate
{
	constexpr int32 MaxInputChannels = 8;
	constexpr int32 MaxNumFramesPerBuffer = 2048;
}

/** Buffered Submix Listener with downmixing and a delegate callback */
FDownmixedBufferedSubmixListener::FDownmixedBufferedSubmixListener(int32 InNumOutputChannels, int32 InSampleRate, const FString* InName)
	: FBufferedSubmixListener{ InNumOutputChannels * InSampleRate, false, InName }
	, NumOutputChannels(InNumOutputChannels)
	, OutputSampleRate(InSampleRate)
	, Resampler(InNumOutputChannels)
{
	ChannelGainMap.Reserve(InNumOutputChannels * DownmixedBufferedSubmixListenerPrivate::MaxInputChannels);
	DownMixBuffer.Reserve(InNumOutputChannels * DownmixedBufferedSubmixListenerPrivate::MaxNumFramesPerBuffer);
	RateConvertedBuffer.Reserve(InNumOutputChannels * DownmixedBufferedSubmixListenerPrivate::MaxNumFramesPerBuffer);
}

int32 FDownmixedBufferedSubmixListener::GetNumAvailableSamples()
{
	return GetNumBufferedSamples();
}

bool FDownmixedBufferedSubmixListener::GetBuffer(float* InBuffer, int32 InBufferSizeInSamples, int32& OutSamplesWritten)
{
	return PopBuffer(InBuffer, InBufferSizeInSamples, OutSamplesWritten);
}

void FDownmixedBufferedSubmixListener::EnableZeroInputBuffer(bool bInZeroInputBuffer)
{
	bZeroInputBuffer = bInZeroInputBuffer;
}

void FDownmixedBufferedSubmixListener::EnableSkipSilentBuffers(bool bInSkipSilentBuffers)
{
	bSkipSilentBuffers = bInSkipSilentBuffers;
}

void FDownmixedBufferedSubmixListener::SetOutputFormat(int32 InNumOutputChannels, int32 InSampleRate)
{
	check(InNumOutputChannels > 0 && InNumOutputChannels <= DownmixedBufferedSubmixListenerPrivate::MaxInputChannels);
	check(InSampleRate > 0);

	// Store the pending format FIRST so the render thread picks it up even if it fires
	// before the flush completes. Worst case: one buffer of new-format data gets flushed
	// (clean underrun / silence), rather than old-format data left in the ring (garbled audio).
	PendingOutputChannels.store(InNumOutputChannels, std::memory_order_relaxed);
	PendingOutputSampleRate.store(InSampleRate, std::memory_order_release);

	// Flush the circular buffer so consumers never read stale old-format data.
	// Grow capacity if the new format requires it (never shrinks).
	const int32 RequiredCapacity = InNumOutputChannels * InSampleRate;
	FlushAndEnsureCapacity(RequiredCapacity);
}

void FDownmixedBufferedSubmixListener::OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 InNumSamples, int32 InNumChannels,
	const int32 InSampleRate, double /*AudioClock*/)
{
	using namespace Audio;

	if (InNumSamples <= 0 || InNumChannels <= 0 || OutputSampleRate <= 0)
	{
		return;
	}
	
	if (bSkipSilentBuffers && IsAudioBufferSilent(AudioData, InNumSamples))
	{
		return;
	}

	if (IsStartedNonAtomic())
	{
		// Pick up a pending output format change from SetOutputFormat()
		const int32 NewOutputSampleRate = PendingOutputSampleRate.exchange(0, std::memory_order_acquire);
		if (NewOutputSampleRate > 0)
		{
			const int32 NewOutputChannels = PendingOutputChannels.load(std::memory_order_relaxed);

			if (NewOutputChannels != NumOutputChannels)
			{
				Resampler.Reset(NewOutputChannels);
			}

			NumOutputChannels = NewOutputChannels;
			OutputSampleRate = NewOutputSampleRate;

			// Force rebuild of input-dependent state on this callback
			InputSampleRate = 0;
			NumInputChannels = 0;
		}

		// Rebuild the sample rate converter if the mixer sample rate changed
		if (InSampleRate != InputSampleRate)
		{
			Resampler.SetFrameRatio((float)InSampleRate / OutputSampleRate);
			InputSampleRate = InSampleRate;
		}

		// Rebuild the downmix gain map if the number of mixer channels has changed
		if (InNumChannels != NumInputChannels)
		{
			check(InNumChannels <= DownmixedBufferedSubmixListenerPrivate::MaxInputChannels);

			FChannelMapParams GainMapParams;
			GainMapParams.NumInputChannels = InNumChannels;
			GainMapParams.NumOutputChannels = NumOutputChannels;
			GainMapParams.Order = EChannelMapOrder::OutputMajorOrder;
			GainMapParams.MonoUpmixMethod = EChannelMapMonoUpmixMethod::EqualPower;
			GainMapParams.bIsCenterChannelOnly = false;

			Create2DChannelMap(GainMapParams, ChannelGainMap);
			NumInputChannels = InNumChannels;
		}

		int32 NumInputFrames = InNumSamples / InNumChannels;
		check(NumInputFrames <= DownmixedBufferedSubmixListenerPrivate::MaxNumFramesPerBuffer);

		const float* DownmixerOutputBuffer = AudioData;

		// Downmix the mixer data if necessary
		if (InNumChannels != NumOutputChannels)
		{
			DownMixBuffer.SetNumUninitialized(NumInputFrames * NumOutputChannels);
			DownmixBuffer(InNumChannels, NumOutputChannels, AudioData, DownMixBuffer.GetData(), NumInputFrames, ChannelGainMap.GetData());
			DownmixerOutputBuffer = DownMixBuffer.GetData();
		}

		// Convert the sample rate if necessary
		int32 NumOutputFrames = NumInputFrames;
		const float* OutputBuffer = DownmixerOutputBuffer;
		if (OutputSampleRate != InSampleRate)
		{
			NumOutputFrames = Resampler.GetNumOutputFramesProducedByInputFrames(NumInputFrames);
			RateConvertedBuffer.SetNumUninitialized(NumOutputFrames * NumOutputChannels);

			int32 NumFramesConsumed = -1;
			int32 NumFramesProduced = -1;

			Resampler.ProcessInterleaved(TArrayView<const float>{DownmixerOutputBuffer, NumInputFrames * NumOutputChannels},
										 TArrayView<float>{RateConvertedBuffer.GetData(), NumOutputFrames * NumOutputChannels},
										 NumFramesConsumed, NumFramesProduced);

			OutputBuffer = RateConvertedBuffer.GetData();
			NumOutputFrames = NumFramesProduced;
		}

		// Call to base class to handle.
		FBufferFormat NewFormat;
		NewFormat.NumChannels = NumOutputChannels;
		NewFormat.NumSamplesPerBlock = NumOutputFrames * NumOutputChannels;
		NewFormat.NumSamplesPerSec = OutputSampleRate;
		OnBufferReceived(NewFormat, MakeArrayView(OutputBuffer, NumOutputFrames * NumOutputChannels));

		// Broadcast to delegates that a buffer has been received
		OnSubmixBufferWritten.ExecuteIfBound(OutputBuffer, NumOutputFrames, NumOutputChannels);
		
		// Optionally, zero the buffer if we're asked to. This in the case where we're running both Unreal+Consumer renderers at once.
		// NOTE: this is dangerous as there's a chance we're not the only listener registered on this Submix. And will cause
		// listeners after us to have a silent buffer. Use with caution. 
		if (bZeroInputBuffer)
		{
			FMemory::Memzero(AudioData, sizeof(float) * InNumSamples);
		}
	}
}

