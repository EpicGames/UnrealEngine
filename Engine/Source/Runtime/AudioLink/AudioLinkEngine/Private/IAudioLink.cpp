// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAudioLink.h"

#include "AudioAnalytics.h"

IAudioLink::IAudioLink()
{
	Audio::Analytics::RecordEvent_Usage(TEXT("AudioLink.InstanceCreated"));
}
