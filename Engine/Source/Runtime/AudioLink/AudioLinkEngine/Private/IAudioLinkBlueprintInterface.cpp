// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAudioLinkBlueprintInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IAudioLinkBlueprintInterface)

UAudioLinkBlueprintInterface::UAudioLinkBlueprintInterface(FObjectInitializer const& InObjectInitializer)
	: UInterface(InObjectInitializer)
{
}

