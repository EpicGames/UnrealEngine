// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAudioLinkFactory.h"
#include "Algo/Transform.h"
#include "AudioLinkLog.h"

// Concrete Buffer Listeners.
#include "BufferedSubmixListener.h" 
#include "BufferedSourceListener.h" 

namespace AudioLinkFactory_Private
{
	bool IsEnabled()
	{
		static bool bAudioLinkEnabled = true;
		static FAutoConsoleVariableRef CVarAudioLinkEnabled(TEXT("au.audiolink.enabled"), bAudioLinkEnabled, TEXT("Enable AudioLink"), ECVF_Default);
		return bAudioLinkEnabled;
	}
}

IAudioLinkFactory::IAudioLinkFactory()
{
	if (AudioLinkFactory_Private::IsEnabled())
	{
		IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
	}
	else
	{
		UE_LOGF(LogAudioLink, Warning, "AudioLink is disabled, au.audiolink.enabled=0. Not registering factory.");
	}
}

IAudioLinkFactory::~IAudioLinkFactory()
{
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
}

FSharedBufferedOutputPtr IAudioLinkFactory::CreateSourceBufferListener(const FSourceBufferListenerCreateParams& InSourceCreateParams)
{	
	auto SourceBufferListenerSP = MakeShared<FBufferedSourceListener, ESPMode::ThreadSafe>(InSourceCreateParams.SizeOfBufferInFrames);
	if (!InSourceCreateParams.AudioComponent.IsExplicitlyNull())
	{
		check(IsInGameThread());
		InSourceCreateParams.AudioComponent->SetSourceBufferListener(SourceBufferListenerSP, InSourceCreateParams.bShouldZeroBuffer);
	}
	return SourceBufferListenerSP;
}

FSharedBufferedOutputPtr IAudioLinkFactory::CreatePushableBufferListener(const FPushedBufferListenerCreateParams& InPushableCreateParams)
{
	// Add push functionality to the source buffer listener with simple wrapper
	struct FPushableSourceBufferListener : FBufferedSourceListener, IPushableAudioOutput
	{
		using FBufferedSourceListener::FBufferedSourceListener;

		IPushableAudioOutput* GetPushableInterface() override { return this; }
		const IPushableAudioOutput* GetPushableInterface() const { return this; }

		void PushNewBuffer(const IPushableAudioOutput::FOnNewBufferParams& InNewBuffer) override
		{
			ISourceBufferListener::FOnNewBufferParams Params;
			Params.AudioData = InNewBuffer.AudioData;
			Params.NumChannels = InNewBuffer.NumChannels;
			Params.NumSamples = InNewBuffer.NumSamples;
			Params.SourceId = InNewBuffer.Id;
			Params.SampleRate = InNewBuffer.SampleRate;
			static_cast<ISourceBufferListener*>(this)->OnNewBuffer(Params);
		}

		void LastBuffer(int32 InId) override
		{
			static_cast<ISourceBufferListener*>(this)->OnSourceReleased(InId);
		}
	};

	auto SourceBufferListenerSP = MakeShared<FPushableSourceBufferListener, ESPMode::ThreadSafe>(InPushableCreateParams.SizeOfBufferInFrames);
	
	return SourceBufferListenerSP;
}

FSharedBufferedOutputPtr IAudioLinkFactory::CreateSubmixBufferListener(const FSubmixBufferListenerCreateParams& InSubmixCreateParams)
{
	const FString ListenerName = FString::Format(TEXT("IAudioLinkFactory:{0}"), { *GetFactoryName().ToString() });
	return MakeShared<FBufferedSubmixListener, ESPMode::ThreadSafe>(InSubmixCreateParams.SizeOfBufferInFrames, InSubmixCreateParams.bShouldZeroBuffer, &ListenerName);
}

TArray<IAudioLinkFactory*> IAudioLinkFactory::GetAllRegisteredFactories()
{
	IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
	return IModularFeatures::Get().GetModularFeatureImplementations<IAudioLinkFactory>(GetModularFeatureName());
}

TArray<FName> IAudioLinkFactory::GetAllRegisteredFactoryNames()
{
	TArray<FName> Names;
	Algo::Transform(GetAllRegisteredFactories(), Names, [](IAudioLinkFactory* Factory) { return Factory->GetFactoryName(); });
	return Names;
}

IAudioLinkFactory* IAudioLinkFactory::FindFactory(const FName InFactoryImplName)
{
	TArray<IAudioLinkFactory*> Factories = GetAllRegisteredFactories();
	if (IAudioLinkFactory** Found = Factories.FindByPredicate([InFactoryImplName](IAudioLinkFactory* Factory) { return Factory->GetFactoryName() == InFactoryImplName; }))
	{
		return *Found;
	}
	return nullptr;
}
