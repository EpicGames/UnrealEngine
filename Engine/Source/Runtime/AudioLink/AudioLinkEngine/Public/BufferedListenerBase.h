// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDevice.h"
#include "DSP/Dsp.h"

#include "IBufferedAudioOutput.h"
#include "Misc/Optional.h"
#include "Math/UnrealMathUtility.h"

// Simple locking circular buffer
// Uses 32bit wrap around logic explained here: 
// https://www.snellman.net/blog/archive/2016-12-13-ring-buffers/
class FLockingCircularSampleBuffer
{
public:
	FLockingCircularSampleBuffer(int32 InInitialCapacity)
	{
		SetCapacity(InInitialCapacity);
	}

	// Allow the caller to lock the CS ahead of calling (if necessary)
	FCriticalSection& GetCriticialSection() { return CS; }
	const FCriticalSection& GetCriticialSection() const { return CS; }

	// This doesn't preserve contents.
	void SetCapacity(int32 InCapacity)
	{
		check(InCapacity > 1);

		if (!FMath::IsPowerOfTwo(InCapacity))
		{
			InCapacity = FMath::RoundUpToPowerOfTwo(InCapacity);
		}

		FScopeLock Lock(&CS);
		Buffer.SetNumZeroed(InCapacity, EAllowShrinking::Yes);
		Mask = InCapacity - 1;
		Read = 0; // Empty.
		Write = 0;

		check(Mask != 0);
	}
	int32 GetCapacity() const
	{
		FScopeLock Lock(&CS);
		return Buffer.Num();
	}
	int32 Num() const
	{
		FScopeLock Lock(&CS);
		return Write - Read;
	}

	// Get number of samples that can be pushed onto the buffer before it is full.
	int32 Remainder() const
	{
		FScopeLock Lock(&CS);
		return Buffer.Num() - Num();
	}

	int32 Push(const float* InBuffer, int32 InSize)
	{
		int32 CanPush = FMath::Min(Remainder(), InSize);
		FScopeLock Lock(&CS);
		for (int32 i = 0; i < CanPush; ++i)
		{
			Enqueue(InBuffer[i]);
		}
		return CanPush;
	}
	int32 Pop(float* OutBuffer, int32 InNumSamples)
	{
		int32 CanPop = FMath::Min(Num(), InNumSamples);
		FScopeLock Lock(&CS);
		for (int32 i = 0; i < CanPop; ++i)
		{
			OutBuffer[i] = Dequeue();
		}
		return CanPop;
	}

	int32 PushZeros(int32 InNumSamplesOfSilence)
	{
		int32 CanPush = FMath::Min(Remainder(), InNumSamplesOfSilence);
		FScopeLock Lock(&CS);
		for (int32 i = 0; i < CanPush; ++i)
		{
			Enqueue(0.f);
		}
		return CanPush;
	}

	// Reset read/write pointers, discarding all buffered data without reallocating.
	void Flush()
	{
		FScopeLock Lock(&CS);
		Read = 0;
		Write = 0;
	}

	// Grow the buffer to at least InMinCapacity if current capacity is insufficient. Never shrinks.
	// Flushes read/write pointers (does not preserve contents).
	void EnsureCapacity(int32 InMinCapacity)
	{
		check(InMinCapacity > 1);

		if (!FMath::IsPowerOfTwo(InMinCapacity))
		{
			InMinCapacity = FMath::RoundUpToPowerOfTwo(InMinCapacity);
		}

		FScopeLock Lock(&CS);
		if (InMinCapacity > Buffer.Num())
		{
			Buffer.SetNumZeroed(InMinCapacity, EAllowShrinking::No);
			Mask = InMinCapacity - 1;
		}
		Read = 0;
		Write = 0;
	}

private:
	uint32 Read = 0;		// These grow indefinitely until wrap at 2^32,
	uint32 Write = 0;		// this allows us to use full capacity as write >= read.
	uint32 Mask = 0;
	TArray<float> Buffer;
	mutable FCriticalSection CS;

	// NOTE: Not forceinline as compiler does better job without.
	void Enqueue(const float InFloat)
	{
		Buffer[Write++ & Mask] = InFloat;
	}
	float Dequeue()
	{
		return Buffer[Read++ & Mask];
	}
};

/** Common base class of Buffered Listener objects.
*/
class FBufferedListenerBase : public IBufferedAudioOutput
{
protected:
	AUDIOLINKENGINE_API FBufferedListenerBase(int32 InDefaultCircularBufferSize);
	virtual ~FBufferedListenerBase() = default;

	//~ Begin IBufferedAudioOutput
	AUDIOLINKENGINE_API bool PopBuffer(float* InBuffer, int32 InBufferSizeInSamples, int32& OutSamplesWritten) override;
	AUDIOLINKENGINE_API bool GetFormat(IBufferedAudioOutput::FBufferFormat& OutFormat) const override;
	AUDIOLINKENGINE_API void Reserve(int32 InNumSamplesToReserve, int32 InNumSamplesOfSilence) override;
	AUDIOLINKENGINE_API void SetFormatKnownDelegate(FOnFormatKnown InFormatKnownDelegate) override;
	void SetBufferStreamEndDelegate(FOnBufferStreamEnd) override {}
	//~ End IBufferedAudioOutput

	//* Common path to receive a new buffer, call from derived classes */
	AUDIOLINKENGINE_API void OnBufferReceived(const FBufferFormat& InFormat, TArrayView<const float> InBuffer);

	//* Reset the format of the buffer */
	AUDIOLINKENGINE_API void ResetFormat();

	//* Set the format of the buffer */
	AUDIOLINKENGINE_API void SetFormat(const FBufferFormat& InFormat);

	//* Ask if the started flag has been set. Note this is non-atomic, as it could change during the call */	
	AUDIOLINKENGINE_API bool IsStartedNonAtomic() const;

	//* Attempt to set our state to started. Not this can fail if we're already started. */	
	AUDIOLINKENGINE_API bool TrySetStartedFlag();

	//* Attempt to set started to false. This can fail if we're already stopped. */	
	AUDIOLINKENGINE_API bool TryUnsetStartedFlag();

	AUDIOLINKENGINE_API bool TrySetStoppingFlag();

	//* Return the number of samples able to be popped off the queue */
	AUDIOLINKENGINE_API int32 GetNumBufferedSamples();

	//* Flush the circular buffer and reset the known format so OnFormatKnown re-fires on the next buffer */
	AUDIOLINKENGINE_API void FlushAndResetFormat();

	//* Flush the circular buffer, grow capacity if needed (never shrinks), and reset format */
	AUDIOLINKENGINE_API void FlushAndEnsureCapacity(int32 InMinCapacity);

private:
	AUDIOLINKENGINE_API void PushSilence(int32 InNumSamplesOfSilence);

	/** Buffer to hold the data for the single source we're listening to, interleaved. */
	FLockingCircularSampleBuffer CircularBuffer;

	/** Read/Write slim lock protects format known optional */
	mutable FRWLock FormatKnownRwLock;

	/** Optional that holds the buffer format, if (and when) it's known. Protected by r/w Slim-lock */
	TOptional<FBufferFormat> KnownFormat;

	/** Delegate that fires when the format it known. Normally on the first buffer received. */
	FOnFormatKnown OnFormatKnown;

	/** Atomic flag we've been started */
	std::atomic<bool> bStarted{ false };

	/** Atomic flag we've been told to stop */
	std::atomic<bool> bStopping{ false };
};

