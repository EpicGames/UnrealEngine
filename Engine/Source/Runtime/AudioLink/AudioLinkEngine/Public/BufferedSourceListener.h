// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDevice.h"
#include "BufferedListenerBase.h"

/**
	FBufferedSourceListener: Concrete implementation of both a buffer listener and buffered output.
	Contains a circular buffer.
*/
class FBufferedSourceListener : public ISourceBufferListener, public FBufferedListenerBase
{
public:
	AUDIOLINKENGINE_API FBufferedSourceListener(int32 InDefaultCircularBufferSize);
	AUDIOLINKENGINE_API virtual ~FBufferedSourceListener();

	/**
	 * @param  InDevice Audio Device to register this listener with.
	 *
	 * @return  success true, false otherwise.
	 */
	AUDIOLINKENGINE_API bool Start(FAudioDevice* InDevice) override;

	/** Stop the buffer by unregistering the buffer listener */
	AUDIOLINKENGINE_API void Stop(FAudioDevice* InDevice) override;

private:
	//~ Begin ISourceBufferListener
	AUDIOLINKENGINE_API void OnNewBuffer(const ISourceBufferListener::FOnNewBufferParams& InParams) override;
	AUDIOLINKENGINE_API void OnSourceReleased(const int32 SourceId) override;
	//~ End ISourceBufferListener

	//~ Begin IBufferedAudioOutput
	AUDIOLINKENGINE_API void SetBufferStreamEndDelegate(FOnBufferStreamEnd InBufferStreamEndDelegate) override;
	//~ End IBufferedAudioOutput

	std::atomic<int32> CurrentSourceId = INDEX_NONE;		// r/w AudioMixer thread.	
	FOnBufferStreamEnd OnBufferStreamEndDelegate;
};
