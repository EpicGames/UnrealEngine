// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BufferedListenerBase.h"
#include "ISubmixBufferListener.h"
#include "Sound/SoundSubmix.h"

/** Concrete Submix Buffer Listener
*/
class FBufferedSubmixListener : public ISubmixBufferListener, public FBufferedListenerBase
{
public:
	/** Constructor
	 * @param  InDefaultCircularBufferSize: Size of the circular buffer in samples by default.
	 * @param  bInZeroInputBuffer: If set, AudioData input buffer will be zeroed out after data is queued
	 * @param  InName(Optional): Optional name to track listener lifetime with.
	 */
	AUDIOLINKENGINE_API FBufferedSubmixListener(int32 InDefaultCircularBufferSize, bool bInZeroInputBuffer, const FString* InName);

	AUDIOLINKENGINE_API virtual ~FBufferedSubmixListener();

	/**
	 * Starts the Submix buffer listener by registering it with the passed in Audio Device.
	 * @param  InDevice: Audio Device to register this submix listener with.
	 * @param  InSubmix: The submix being listened to (default is main output)
	 *
	 * @return  success true, false otherwise.
	 */
	AUDIOLINKENGINE_API bool Start(FAudioDevice* InDevice, USoundSubmix* InSubmix);
	//~ Begin IBufferedAudioOutput
	AUDIOLINKENGINE_API bool Start(FAudioDevice* InDevice) override;
	AUDIOLINKENGINE_API void Stop(FAudioDevice* InDevice) override;
	//~ End IBufferedAudioOutput

protected:
	//~ Begin ISubmixBufferListener
	AUDIOLINKENGINE_API void OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 InNumSamples, int32 InNumChannels, const int32 InSampleRate, double) override;
	AUDIOLINKENGINE_API const FString& GetListenerName() const override;
	//~ End ISubmixBufferListener

	bool bZeroInputBuffer = false;
private:
	Audio::FDeviceId DeviceId;
	FString Name;

	TWeakObjectPtr<USoundSubmix> Submix;
};
