// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>

#include "AudioDevice.h"
#include "BufferedSubmixListener.h"
#include "DSP/RuntimeResampler.h"

class USoundSubmix;

class FDownmixedBufferedSubmixListener : public FBufferedSubmixListener
{
public:
	/** Constructor
	 * @param  InNumOutputChannels: channel config for output
	 * @param  InSampleRate: Desired sample rate
	 * @param  InName(Optional): Optional name to track listener lifetime with.
	 */
	AUDIOLINKENGINE_API FDownmixedBufferedSubmixListener(int32 InNumOutputChannels, int32 InSampleRate, const FString* InName);

	virtual ~FDownmixedBufferedSubmixListener() {}

	// Delegate to signal to the consumer that data is ready
	DECLARE_DELEGATE_ThreeParams(FOnSubmixBufferWritten, const float* /*SampleBuffer*/, int32 /*FrameCount*/, int32 /*NumChannels*/);
	FOnSubmixBufferWritten OnSubmixBufferWritten;

	/** Query the number of samples waiting in the queue
	 * @return total number of samples, e.g. for 480 frames of 2 channels, would return 960
	 */
	AUDIOLINKENGINE_API int32 GetNumAvailableSamples();

	/** Retrieve samples from the queue
	 * @param  InBuffer: memory provided by the caller to copy samples into
	 * @param  InBufferSizeInSamples: total number of samples to be retrieved, frame count * channel count
	 * @param  OutSamplesWritten: total number of samples written into the provided buffer
	 * 
	 * @return if the queue is still active, it will return true, otherwise, return indicates if there are more samples in the queue
	 */
	AUDIOLINKENGINE_API bool GetBuffer(float* InBuffer, int32 InBufferSizeInSamples, int32& OutSamplesWritten);
	
	/** If set to true, AudioData input buffer will be zeroed out after data is queued.
	 * NOTE: Only set this to true if you can be sure that this is the only listener registered on this Submix.
	 * As such, listeners after this will have a silent buffer and the Submix may be considered disabled if bAutoDisable is on for that Submix. Use with caution. 
	 * @param  bInZeroInputBuffer: bool value to set bZeroInputBuffer to
	 */
	AUDIOLINKENGINE_API void EnableZeroInputBuffer(bool bInZeroInputBuffer);
	
	/** If set to true, only buffers that contain at least one non-silent sample will be written to the circular buffer.
	 * @param  bInSkipSilentBuffers: bool value to set bSkipSilentBuffers to
	 */
	AUDIOLINKENGINE_API void EnableSkipSilentBuffers(bool bInSkipSilentBuffers);

	/** Change the output format of the listener. Safe to call from any thread.
	 * Flushes the circular buffer immediately so that subsequent PopBuffer()
	 * calls will not return stale data in the old format.
	 * The audio render thread will pick up the new format on the next
	 * OnNewSubmixBuffer() callback and rebuild internal DSP state accordingly.
	 * Callers should expect up to one buffer period of silence after the transition.
	 *
	 * @param  InNumOutputChannels: New output channel count (max 8)
	 * @param  InSampleRate: New output sample rate
	 */
	AUDIOLINKENGINE_API void SetOutputFormat(int32 InNumOutputChannels, int32 InSampleRate);

private:
	//~ Begin ISubmixBufferListener
	AUDIOLINKENGINE_API void OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 InNumSamples, int32 InNumChannels, const int32 InSampleRate, double) override;
	//~ End ISubmixBufferListener

	int32 NumOutputChannels;
	int32 OutputSampleRate;
	bool bSkipSilentBuffers = false;

	// Pending output format change, set by SetOutputFormat(), consumed by OnNewSubmixBuffer()
	std::atomic<int32> PendingOutputChannels{0};
	std::atomic<int32> PendingOutputSampleRate{0};  // 0 = no pending change (gate variable)

	TArray<float> ChannelGainMap;
	Audio::FAlignedFloatBuffer DownMixBuffer;
	Audio::FAlignedFloatBuffer RateConvertedBuffer;
	Audio::FRuntimeResampler Resampler;
	int32 NumInputChannels = 0;
	int32 InputSampleRate = 0;
};
