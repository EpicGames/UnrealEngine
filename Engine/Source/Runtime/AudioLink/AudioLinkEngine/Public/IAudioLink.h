// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/Transform.h"
#include "Delegates/DelegateCombinations.h"

/** IAudioLink
  *	Abstract interface for AudioLink instances. 
  *	Purely opaque.
  */
class IAudioLink
{
protected:
	AUDIOLINKENGINE_API IAudioLink();
public:
	virtual ~IAudioLink() = default;
};

/** IAudioLinkSourcePushed
  *	Where the owning object needs to push it's state
  */
class IAudioLinkSourcePushed : public IAudioLink
{
protected:
	IAudioLinkSourcePushed() = default;
public:
	virtual ~IAudioLinkSourcePushed() = default;

	struct FOnUpdateWorldStateParams
	{
		FTransform	WorldTransform;
	};
	virtual void OnUpdateWorldState(const FOnUpdateWorldStateParams&) = 0;

	struct FOnNewBufferParams
	{
		TArrayView<float> Buffer;
		int32 SourceId = INDEX_NONE;
	};
	virtual void OnNewBuffer(const FOnNewBufferParams&) = 0;

	virtual void OnSourceDone(const int32 SourceId) = 0;

	virtual void OnSourceReleased(const int32 SourceId) = 0;


};

/** IAudioLinkSynchronizer
  *	Provides delegates for hooking and external AudioLinks synchronization callbacks.
  */
class IAudioLinkSynchronizer
{
protected:
	IAudioLinkSynchronizer() = default;
public:
	virtual ~IAudioLinkSynchronizer() = default;	
	
	DECLARE_MULTICAST_DELEGATE(FOnSuspend);
	virtual FDelegateHandle RegisterSuspendDelegate(const FOnSuspend::FDelegate&) = 0;
	virtual bool RemoveSuspendDelegate(const FDelegateHandle&) = 0;

	DECLARE_MULTICAST_DELEGATE(FOnResume);
	virtual FDelegateHandle RegisterResumeDelegate(const FOnResume::FDelegate&) = 0;
	virtual bool RemoveResumeDelegate(const FDelegateHandle&) = 0;

	struct FOnOpenStreamParams
	{
		/**!
		 * Name of Implementation.
		 */
		FString Name;

		/**
		 * Number of frames per callback buffer. INDEX_NONE if not set.
		 */
		int32 NumFrames = INDEX_NONE;

		/**
		 * Number of channels we're rendering at i.e. Stereo=2. INDEX_NONE if not set.
		 */
		int32 NumChannels = INDEX_NONE;

		/**
		 * Sample rate of AL implementation. INDEX_NONE if not set.
		 */
		int32 SampleRate = INDEX_NONE;

		/**
		 * Num of Sources that AL is limited to. INDEX_NONE if not set.
		 */
		int32 NumSources = INDEX_NONE;
	};

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnOpenStream, const FOnOpenStreamParams&);
	virtual FDelegateHandle RegisterOpenStreamDelegate(const FOnOpenStream::FDelegate&) = 0;
	virtual bool RemoveOpenStreamDelegate(const FDelegateHandle&) = 0;

	DECLARE_MULTICAST_DELEGATE(FOnCloseStream);
	virtual FDelegateHandle RegisterCloseStreamDelegate(const FOnCloseStream::FDelegate&) = 0;
	virtual bool RemoveCloseStreamDelegate(const FDelegateHandle&) = 0;

	struct FOnRenderParams 
	{
		uint64 BufferTickID = 0;
		int32 NumFrames = INDEX_NONE;
	};
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBeginRender, const FOnRenderParams&);
	virtual FDelegateHandle RegisterBeginRenderDelegate(const FOnBeginRender::FDelegate&) = 0;
	virtual bool RemoveBeginRenderDelegate(const FDelegateHandle&) = 0;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEndRender, const FOnRenderParams&);
	virtual FDelegateHandle RegisterEndRenderDelegate(const FOnEndRender::FDelegate&) = 0;
	virtual bool RemoveEndRenderDelegate(const FDelegateHandle&) = 0;

	/**
	 * Gets any OpenStreamParams that were sent ahead of synchronizing existing.
	 * @return TOptional<FOpenStreamParams> (set if data is available).
	 */
	virtual TOptional<FOnOpenStreamParams> GetCachedOpenStreamParams() const { return {}; }
};
