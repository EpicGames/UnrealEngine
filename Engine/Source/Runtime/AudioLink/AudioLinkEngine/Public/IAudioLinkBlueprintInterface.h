// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "IAudioLinkBlueprintInterface.generated.h"

class UObject;
class USoundBase;
struct FFrame;

UINTERFACE(BlueprintType, meta = (CannotImplementInterfaceInBlueprint), MinimalAPI)
class UAudioLinkBlueprintInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IAudioLinkBlueprintInterface : public IInterface
{
	GENERATED_IINTERFACE_BODY()

public:	
	UFUNCTION(BlueprintCallable, Category = "AudioLink")
	virtual void SetLinkSound(USoundBase* NewSound) = 0;

	UFUNCTION(BlueprintCallable, Category = "AudioLink")
	virtual void PlayLink(float StartTime = 0.0f) = 0;

	/** Stop an audio component's sound, issue any delegates if needed */
	UFUNCTION(BlueprintCallable, Category = "AudioLink")
	virtual void StopLink() = 0;
		
	UFUNCTION(BlueprintCallable, Category = "AudioLink")
	virtual bool IsLinkPlaying() const = 0;
};

