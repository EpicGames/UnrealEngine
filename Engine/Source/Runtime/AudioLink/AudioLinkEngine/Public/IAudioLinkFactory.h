// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioLink.h"
#include "Features/IModularFeatures.h"
#include "AudioDeviceManager.h"
#include "AudioLinkSettingsAbstract.h"
#include "Templates/SubclassOf.h"
#include "IBufferedAudioOutput.h"

// Forward declarations.
class USoundSubmix;
class UAudioLinkComponent;

/**
	IAudioLinkFactory
	Abstract interface for AudioLink factory objects. 
*/
class IAudioLinkFactory : public IModularFeature
{
protected:
	/** Constructor will auto register this instance. */
	AUDIOLINKENGINE_API IAudioLinkFactory();
public:
	/** Destructor will auto unregister this instance. */
	AUDIOLINKENGINE_API virtual ~IAudioLinkFactory();

	/** 
	 * Gets the name of this factory type. This will uniquely identity itself, so it can be found by the FindFactory call below
	 * @return Name of factory
	 */
	virtual FName GetFactoryName() const = 0;

	/**
	 * Gets the type of of the settings object that this factory uses to define its specific settings. All settings for each
	 * AudioLink factory will derive from UAudioLinkSettingsAbstract
	 * @return Class of the Settings Object
	 */
	virtual TSubclassOf<UAudioLinkSettingsAbstract> GetSettingsClass() const = 0;
	
	struct FSourceBufferListenerCreateParams
	{
		int32 SizeOfBufferInFrames = 0;
		bool bShouldZeroBuffer = false;
		TWeakObjectPtr<UAudioComponent> AudioComponent;
		TWeakObjectPtr<USceneComponent> OwningComponent;
	};
	AUDIOLINKENGINE_API virtual FSharedBufferedOutputPtr CreateSourceBufferListener(const FSourceBufferListenerCreateParams&);

	struct FPushedBufferListenerCreateParams
	{
		int32 SizeOfBufferInFrames = INDEX_NONE;
		bool bShouldZeroBuffer = false;
	};
	AUDIOLINKENGINE_API virtual FSharedBufferedOutputPtr CreatePushableBufferListener(const FPushedBufferListenerCreateParams&);

	struct FSubmixBufferListenerCreateParams
	{
		int32 SizeOfBufferInFrames = 0;
		bool bShouldZeroBuffer = false;
	};
	AUDIOLINKENGINE_API virtual FSharedBufferedOutputPtr CreateSubmixBufferListener(const FSubmixBufferListenerCreateParams&);
			
	/**
	 * Parameters use when creating a Submix Audio Link 
	 */
	struct FAudioLinkSubmixCreateArgs
	{
		TWeakObjectPtr<const USoundSubmix> Submix;
		FAudioDevice* Device = nullptr;
		TWeakObjectPtr<const UAudioLinkSettingsAbstract> Settings;
	};

	/**
	 * Create a Submix Audio Link.
	 * @param InCreateArgs Arguments used to create the AudioLink instance
	 * @return The newly created Link instance (if successful).
	 */
	virtual TUniquePtr<IAudioLink> CreateSubmixAudioLink(const FAudioLinkSubmixCreateArgs& InCreateArgs) = 0;

	/**
	 * Parameters use when creating a Source Audio Link 
	 */
	struct FAudioLinkSourceCreateArgs
	{
		TWeakObjectPtr<UAudioComponent> AudioComponent;
		TWeakObjectPtr<USceneComponent> OwningComponent;
		TWeakObjectPtr<UAudioLinkSettingsAbstract> Settings;
	};

	/**
	 * Create a Source Audio Link.
	 * @param InCreateArgs Arguments used to create the AudioLink instance
	 * @return The newly created Link instance (if successful).
	 */
	virtual TUniquePtr<IAudioLink> CreateSourceAudioLink(const FAudioLinkSourceCreateArgs&) = 0;
	
	struct FAudioLinkSourcePushedCreateArgs
	{
		FName OwnerName;
		int32 NumChannels = INDEX_NONE;
		int32 SampleRate = INDEX_NONE;
		int32 NumFramesPerBuffer = INDEX_NONE;
		int32 TotalNumFramesInSource = INDEX_NONE;
		UAudioLinkSettingsAbstract::FSharedSettingsProxyPtr Settings;
	};	
	using FAudioLinkSourcePushedSharedPtr = TSharedPtr<IAudioLinkSourcePushed, ESPMode::ThreadSafe>;
	virtual FAudioLinkSourcePushedSharedPtr CreateSourcePushedAudioLink(const FAudioLinkSourcePushedCreateArgs&) = 0;

	/**
	 * Create a AudioLinkSynchronizer callback
	 * @return The newly created Link instance (if successful).
	 */
	using FAudioLinkSynchronizerSharedPtr = TSharedPtr<IAudioLinkSynchronizer, ESPMode::ThreadSafe>;
	virtual FAudioLinkSynchronizerSharedPtr CreateSynchronizerAudioLink() = 0;
		
	/**	
	 * Get the name of all AudioLink factories in the Modular Features registry.
	 * @return "AudioLink Factory"
	 */

	inline static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("AudioLink Factory");
		return FeatureName;
	}

	/**
	 * Gets all registered factory instances.
	 * @return Array of all factories.
	 */
	static AUDIOLINKENGINE_API TArray<IAudioLinkFactory*> GetAllRegisteredFactories();

	/**
	 * Gets all registered factory names
	 * @return Array of all factory names
	 */
	static AUDIOLINKENGINE_API TArray<FName> GetAllRegisteredFactoryNames();
	
	/**
	 * Gets all registered factory names
	 * @return Array of all factory names
	 */
	static AUDIOLINKENGINE_API IAudioLinkFactory* FindFactory(const FName InFactoryName);

protected:	
};

