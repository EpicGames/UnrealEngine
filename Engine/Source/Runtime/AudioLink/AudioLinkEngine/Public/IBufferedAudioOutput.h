// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"

class FAudioDevice;

class IPushableAudioOutput
{
public:
	struct FOnNewBufferParams
	{
		const float* AudioData = nullptr;
		int32 Id = INDEX_NONE;
		int32 NumSamples = INDEX_NONE;
		int32 NumChannels = INDEX_NONE;
		int32 SampleRate = INDEX_NONE;
	};
	virtual void PushNewBuffer(const FOnNewBufferParams&) = 0;	
	virtual void LastBuffer(int32 InId) = 0;
};

/**
	Abstract interface for communication with outputting audio objects
	Examples concrete implementation of these are (source, submix etc).
*/

class IBufferedAudioOutput 
{
protected:
	IBufferedAudioOutput() = default;

public:
	virtual ~IBufferedAudioOutput() = default;

	/**
	 * The format of a buffer.
	 */
	struct FBufferFormat
	{
		int32 NumSamplesPerBlock = 0;
		int32 NumChannels = 0;
		int32 NumSamplesPerSec = 0;

		bool operator==(const FBufferFormat& InRhs) const
		{
			return InRhs.NumChannels == NumChannels && 
				InRhs.NumSamplesPerBlock == NumSamplesPerBlock && 
				InRhs.NumSamplesPerSec == NumSamplesPerSec;
		}
	};

	// Delegates.
	DECLARE_DELEGATE_OneParam(FOnFormatKnown, FBufferFormat);
	virtual void SetFormatKnownDelegate(FOnFormatKnown InFormatKnownDelegate) = 0;

	/**
	 * Stream of buffers ended.
	 */
	struct FBufferStreamEnd
	{
		int32 Id = INDEX_NONE;
	};

	// Delegates.
	DECLARE_DELEGATE_OneParam(FOnBufferStreamEnd, FBufferStreamEnd);
	virtual void SetBufferStreamEndDelegate(FOnBufferStreamEnd InBufferStreamEndDelegate) = 0;

	virtual bool Start(FAudioDevice* InAudioDevice) = 0;
	virtual void Stop(FAudioDevice* InAudioDevice) = 0;

	/**
	 * Attempts to Atomically copy a buffer sized amount of Buffered Sample data from the interface.
	 * 
	 * @param InBuffer Sample Buffer to Write to
	 * @param InBufferSizeInSamples Buffer Size in total samples. 
	 * @param OutSamplesWritten Number of Samples Written to the buffer
	 * 
	 * @return false no more data, true more data to come
	 */
	virtual bool PopBuffer(float* InBuffer, int32 InBufferSizeInSamples, int32& OutSamplesWritten) = 0;

	/**
	 * Gets the format of the buffer, if its known.
	 *
	 * @return  success true, false otherwise. 
	 */
	virtual bool GetFormat(FBufferFormat& OutFormat) const = 0;

	/**
	 * Reserve at least this many samples in buffer.
	 *
	 * @param InNumSamplesToReserve Reserve this number of samples.
	 */
	virtual void Reserve(int32 InNumSamplesToReserve, int32 InNumSamplesOfSilence = 0) = 0;

	virtual IPushableAudioOutput* GetPushableInterface() { return nullptr; }
	virtual const IPushableAudioOutput* GetPushableInterface() const { return nullptr; }
};

using FWeakBufferedOutputPtr = TWeakPtr<IBufferedAudioOutput, ESPMode::ThreadSafe>;
using FSharedBufferedOutputPtr = TSharedPtr<IBufferedAudioOutput, ESPMode::ThreadSafe>;

