// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioMixerPlatformAudioLink: ModuleRules
{
	public AudioMixerPlatformAudioLink(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateIncludePathModuleNames.Add("TargetPlatform");
		PublicIncludePathModuleNames.Add("AudioMixer");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AudioMixer",
				"AudioMixerCore",
				"AudioLinkEngine",
				"Core",
				"CoreUObject",
				"Engine",
		});
					
	}
}
