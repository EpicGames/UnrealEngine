// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerPlatformAudioLink.h"
#include "Modules/ModuleManager.h"
#include "AudioMixer.h"
#include "AudioMixerDevice.h"
#include "CoreGlobals.h"
#include "Misc/ConfigCacheIni.h"

#include "Misc/CoreMisc.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "AudioLinkLog.h"

namespace Audio
{
	// We can't easily access AudioMixer instance from here, so find the factory in the registry.
	static IAudioLinkFactory* GetAudioLinkFactory()
	{
		TArray<FName> Names = IAudioLinkFactory::GetAllRegisteredFactoryNames();
		if (Names.Num())
		{
			return IAudioLinkFactory::FindFactory(Names[0]);
		}
		return nullptr;
	}

	FAudioMixerPlatformAudioLink::FAudioMixerPlatformAudioLink()
		: Factory(GetAudioLinkFactory())
	{
		MakeDeviceInfo(
			8,
			48000,
			TEXT("AudioLink AudioMixer")
		);
	}

	bool FAudioMixerPlatformAudioLink::InitializeHardware()
	{
		if (IAudioMixer::ShouldRecycleThreads())
		{
			// Pre-create the null render device thread, so we can simple wake it up when we need it.
			// Give it nothing to do, with a slow tick as the default, but ask it to wait for a signal to wake up.
			CreateNullDeviceThread([] {}, 1.0f, true);
		}

		if (IAudioLinkSynchronizer* Synchronizer = GetOrCreateSynchronizer())
		{
			using FThisType = FAudioMixerPlatformAudioLink;
			using FSync = IAudioLinkSynchronizer;
		
			// Register callbacks.
			RenderBeginHandle = Synchronizer->RegisterBeginRenderDelegate(FSync::FOnBeginRender::FDelegate::CreateRaw(this, &FThisType::OnLinkRenderBegin));
			RenderEndHandle = Synchronizer->RegisterEndRenderDelegate(FSync::FOnEndRender::FDelegate::CreateRaw(this, &FThisType::OnLinkRenderEnd));
			OpenStreamHandle = Synchronizer->RegisterOpenStreamDelegate(FSync::FOnOpenStream::FDelegate::CreateRaw(this, &FThisType::OnLinkOpenStream));

			if (const TOptional Params = Synchronizer->GetCachedOpenStreamParams(); Params.IsSet())
			{
				OnLinkOpenStream(*Params);
			}
		}

		bInitialized = true;
		return true;
	}

	bool FAudioMixerPlatformAudioLink::TeardownHardware()
	{		
		StopAudioStream();
		CloseAudioStream();

		// Kill synchronizer, and unregister callbacks.
		if (SynchronizeLink.IsValid())
		{
			SynchronizeLink->RemoveBeginRenderDelegate(RenderBeginHandle);
			SynchronizeLink->RemoveEndRenderDelegate(RenderEndHandle);
			SynchronizeLink->RemoveOpenStreamDelegate(OpenStreamHandle);
			RenderBeginHandle.Reset();
			RenderEndHandle.Reset();
			OpenStreamHandle.Reset();
		}
		SynchronizeLink.Reset();
		return true;
	}

	bool FAudioMixerPlatformAudioLink::IsInitialized() const
	{
		return bInitialized;
	}

	bool FAudioMixerPlatformAudioLink::GetNumOutputDevices(uint32& OutNumOutputDevices)
	{
		OutNumOutputDevices = 1;
		return true;
	}
	
	bool FAudioMixerPlatformAudioLink::GetOutputDeviceInfo(const uint32 InDeviceIndex, FAudioPlatformDeviceInfo& OutInfo)
	{	
		OutInfo = DeviceInfo;
		return true;
	}

	bool FAudioMixerPlatformAudioLink::GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const
	{
		// It's not possible to know what index the default audio device is.
		OutDefaultDeviceIndex = AUDIO_MIXER_DEFAULT_DEVICE_INDEX;
		return true;
	}

	bool FAudioMixerPlatformAudioLink::OpenAudioStream(const FAudioMixerOpenStreamParams& Params)
	{
		if (!bInitialized || AudioStreamInfo.StreamState != EAudioOutputStreamState::Closed)
		{
			return false;
		}
			
		AudioStreamInfo.Reset();
		if(!GetOutputDeviceInfo(Params.OutputDeviceIndex, AudioStreamInfo.DeviceInfo))
		{
			return false;
		}

		OpenStreamParams = Params;

		AudioStreamInfo.AudioMixer = Params.AudioMixer;
		AudioStreamInfo.NumBuffers = Params.NumBuffers;
		AudioStreamInfo.NumOutputFrames = Params.NumFrames;
		AudioStreamInfo.StreamState = EAudioOutputStreamState::Open;

		return true;
	}

	bool FAudioMixerPlatformAudioLink::CloseAudioStream()
	{
		if (AudioStreamInfo.StreamState == EAudioOutputStreamState::Closed)
		{
			return false;
		}

		if (!StopAudioStream())
		{
			return false;
		}

		AudioStreamInfo.StreamState = EAudioOutputStreamState::Closed;
		return true;
	}

	bool FAudioMixerPlatformAudioLink::StartAudioStream()
	{
		if (!bInitialized || (AudioStreamInfo.StreamState != EAudioOutputStreamState::Open && AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopped))
		{
			return false;
		}

		// Start generating audio
		BeginGeneratingAudio();

		if (!SynchronizeLink.IsValid())
		{
			StartRunningNullDevice();
		}

		AudioStreamInfo.StreamState = EAudioOutputStreamState::Running;

		bAtomicStreamRunning = true; 

		return true;
	}

	bool FAudioMixerPlatformAudioLink::StopAudioStream()
	{
		if (AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopped && AudioStreamInfo.StreamState != EAudioOutputStreamState::Closed)
		{
			if(bIsUsingNullDevice)
			{
				StopRunningNullDevice();
			}
			
			if (AudioStreamInfo.StreamState == EAudioOutputStreamState::Running)
			{
				StopGeneratingAudio();
			}

			check(AudioStreamInfo.StreamState == EAudioOutputStreamState::Stopped);
		}
		
		bAtomicStreamRunning = false;

		return true;
	}

	FAudioPlatformDeviceInfo FAudioMixerPlatformAudioLink::GetPlatformDeviceInfo() const
	{
		return AudioStreamInfo.DeviceInfo;
	}

	FString FAudioMixerPlatformAudioLink::GetDefaultDeviceName()
	{
		return DeviceInfo.Name;
	}

	IAudioLinkSynchronizer* FAudioMixerPlatformAudioLink::GetOrCreateSynchronizer() const
	{
		if (!SynchronizeLink.IsValid())
		{
			check(Factory);
			SynchronizeLink = Factory->CreateSynchronizerAudioLink();
		}
		return SynchronizeLink.Get();
	}
	
	FAudioPlatformSettings FAudioMixerPlatformAudioLink::GetPlatformSettings() const
	{
#if WITH_ENGINE
		FAudioPlatformSettings Settings = FAudioPlatformSettings::GetPlatformSettings(FPlatformProperties::GetRuntimeSettingsClassName());
		
		if (const IAudioLinkSynchronizer* Synchronizer = GetOrCreateSynchronizer())
		{
			if (TOptional<IAudioLinkSynchronizer::FOnOpenStreamParams> Params = Synchronizer->GetCachedOpenStreamParams(); Params.IsSet())
			{
				// Override the .INI settings with the host audio settings, assuming we're set up.
				// Check if each setting has been passed by AudioLink implementation. It will be INDEX_NONE otherwise.

				// NOTE. 'MaxChannels' on the settings is really MaxSources.
				if (Params->NumSources != INDEX_NONE)
				{
					Settings.MaxChannels = Params->NumSources; 
				}
				if (Params->SampleRate != INDEX_NONE)
				{
					Settings.SampleRate = Params->SampleRate; 
				}
				if (Params->NumFrames != INDEX_NONE) 
				{
					Settings.CallbackBufferFrameSize = Params->NumFrames;
				}
			}
			else
			{
				UE_LOGF(LogAudioLink, Warning, "%s - OpenStreamParams has not been set, all settings will be from the Default Engine settings, not the host.", __func__);
			}
		}
		else
		{
			UE_LOGF(LogAudioLink, Warning, "%s - Synchronizer does not exist yet.", __func__);
		}
		return Settings;;
#else
		return {};
#endif // WITH_ENGINE
	}

	void FAudioMixerPlatformAudioLink::MakeDeviceInfo(int32 InNumChannels, int32 InSampleRate, const FString& InName)
	{
		DeviceInfo.Reset();
		DeviceInfo.Name = InName;
		DeviceInfo.DeviceId = InName;
		DeviceInfo.SampleRate = InSampleRate;
		DeviceInfo.NumChannels = InNumChannels;
		DeviceInfo.bIsSystemDefault = true;
		DeviceInfo.Format = EAudioMixerStreamDataFormat::Float;
		
		DeviceInfo.OutputChannelArray.SetNum(InNumChannels);
		for (int32 i = 0; i < InNumChannels; ++i)
		{
			DeviceInfo.OutputChannelArray.Add(EAudioMixerChannel::Type(i));
		}
	}

	void FAudioMixerPlatformAudioLink::OnLinkOpenStream(const IAudioLinkSynchronizer::FOnOpenStreamParams& InParams)
	{
		MakeDeviceInfo(
			InParams.NumChannels,
			InParams.SampleRate,
			TEXT("AudioLink AudioMixer"));

	}

	void FAudioMixerPlatformAudioLink::OnLinkRenderBegin(const IAudioLinkSynchronizer::FOnRenderParams& InParams)
	{		
	}
	void FAudioMixerPlatformAudioLink::OnLinkRenderEnd(const IAudioLinkSynchronizer::FOnRenderParams& InParams)
	{
		UE_LOGF(LogAudioLink, VeryVerbose, "FAudioMixerPlatformAudioLink::OnLinkRenderEnd, TickID=%llu, FramesMade=%d, LastBufferTickID=%d, FrameRemainder=%d, AudioMixer.NumFrames=%d, AudioMixer.NumBuffers=%dm This=0x%p", 
				InParams.BufferTickID, InParams.NumFrames, LastBufferTickID, FrameRemainder, AudioStreamInfo.NumOutputFrames, AudioStreamInfo.NumBuffers, this);

		if (bAtomicStreamRunning)  // Make sure Unreal is open and ready to receive input.
		{
			if (LastBufferTickID < InParams.BufferTickID)
			{
				LastBufferTickID = InParams.BufferTickID;

				FrameRemainder += InParams.NumFrames;

				if (FrameRemainder >= AudioStreamInfo.NumOutputFrames)
				{
					FrameRemainder = 0;
					ReadNextBuffer();
				}
			}
		}
	}
}
