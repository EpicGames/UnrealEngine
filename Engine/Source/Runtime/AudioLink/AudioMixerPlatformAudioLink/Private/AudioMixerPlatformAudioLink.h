// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"
#include "IAudioLinkFactory.h"

namespace Audio
{
	class FAudioMixerPlatformAudioLink : public IAudioMixerPlatformInterface
	{
	public:

		FAudioMixerPlatformAudioLink();
		virtual ~FAudioMixerPlatformAudioLink() = default;
	protected:

		//~ Begin IAudioMixerPlatformInterface Interface
		FString GetPlatformApi() const override { return TEXT("AudioLink"); }
		bool InitializeHardware() override;
		bool TeardownHardware() override;
		bool IsInitialized() const override;
		bool GetNumOutputDevices(uint32& OutNumOutputDevices) override;
		bool GetOutputDeviceInfo(const uint32 InDeviceIndex, FAudioPlatformDeviceInfo& OutInfo) override;
		bool GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const override;
		virtual bool OpenAudioStream(const FAudioMixerOpenStreamParams& Params) override;
		bool CloseAudioStream() override;
		bool StartAudioStream() override;
		bool StopAudioStream() override;
		FAudioPlatformDeviceInfo GetPlatformDeviceInfo() const override;
		FString GetDefaultDeviceName() override;
		FAudioPlatformSettings GetPlatformSettings() const override;
		//~ End IAudioMixerPlatformInterface Interface

	private:
		void MakeDeviceInfo(int32 InNumChannels, int32 InSampleRate, const FString& InName);

		void OnLinkOpenStream(const IAudioLinkSynchronizer::FOnOpenStreamParams&);
		void OnLinkRenderBegin(const IAudioLinkSynchronizer::FOnRenderParams&);
		void OnLinkRenderEnd(const IAudioLinkSynchronizer::FOnRenderParams&);

		IAudioLinkSynchronizer* GetOrCreateSynchronizer() const;

		bool bSuspended = false;
		bool bInitialized = false;
		
		// Atomic flag that's set by unreal, but read in callback from other host engines.
		std::atomic<bool> bAtomicStreamRunning;
		
		IAudioLinkFactory* Factory = nullptr;
		mutable IAudioLinkFactory::FAudioLinkSynchronizerSharedPtr SynchronizeLink;
		FAudioPlatformDeviceInfo DeviceInfo;
		uint32 LastBufferTickID = 0;
		int32 FrameRemainder = 0;

		FDelegateHandle OpenStreamHandle;
		FDelegateHandle RenderBeginHandle;
		FDelegateHandle RenderEndHandle;
	};

}

