// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "AudioMixer.h"
#include "AudioMixerDevice.h"
#include "AudioMixerPlatformAudioLink.h"

class FAudioMixerPlatformAudioLinkModule : public IAudioDeviceModule
{
public:
	virtual bool IsAudioMixerModule() const override { return true; }

	virtual void StartupModule() override
	{
		IAudioDeviceModule::StartupModule();

		FModuleManager::Get().LoadModuleChecked(TEXT("AudioMixerCore"));
	}

	virtual FAudioDevice* CreateAudioDevice() override
	{
		return new Audio::FMixerDevice(CreateAudioMixerPlatformInterface());
	}

	virtual Audio::IAudioMixerPlatformInterface* CreateAudioMixerPlatformInterface() override
	{
		return new Audio::FAudioMixerPlatformAudioLink();
	}
};

IMPLEMENT_MODULE(FAudioMixerPlatformAudioLinkModule, AudioMixerPlatformAudioLink);
